[/Script/MacTargetPlatform.XcodeProjectSettings]
TemplateMacPlist=(FilePath="/Engine/Source/Programs/UnrealBuildAccelerator/Visualizer/Resources/Mac/Info.Template.plist")
